// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DeviceManager : ModuleRules
{
	public DeviceManager(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"TargetDeviceServices",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"InputCore",
				"Slate",
				"SlateCore",
				"TargetPlatform",
				"DesktopPlatform",
                "WorkspaceMenuStructure",
			}
		);
	}
}
