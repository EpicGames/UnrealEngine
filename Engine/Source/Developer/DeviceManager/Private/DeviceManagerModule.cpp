// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Styling/AppStyle.h"
#include "Framework/Docking/TabManager.h"
#include "IDeviceManagerModule.h"
#include "ITargetDeviceServicesModule.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SDeviceManager.h"
#include "Widgets/SWidget.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"


static const FName DeviceManagerTabName("DeviceManager");


/**
 * Implements the DeviceManager module.
 */
class FDeviceManagerModule
	: public IDeviceManagerModule
{
public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// @todo gmp: implement an IoC container
		ITargetDeviceServicesModule& TargetDeviceServicesModule = FModuleManager::LoadModuleChecked<ITargetDeviceServicesModule>(TEXT("TargetDeviceServices"));

		TargetDeviceServiceManager = TargetDeviceServicesModule.GetDeviceServiceManager();

		auto& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(DeviceManagerTabName, FOnSpawnTab::CreateRaw(this, &FDeviceManagerModule::SpawnDeviceManagerTab))
			.SetDisplayName(NSLOCTEXT("FDeviceManagerModule", "DeviceManagerTabTitle", "Device Manager"))
			.SetTooltipText(NSLOCTEXT("FDeviceManagerModule", "DeviceManagerTooltipText", "View and manage connected devices."))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "DeviceDetails.TabIcon"));

#if WITH_EDITOR
		TabSpawnerEntry.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsPlatformsCategory());
#else
		TabSpawnerEntry.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory());
#endif
	}

	virtual void ShutdownModule() override
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(DeviceManagerTabName);
	}

public:

	//~ IDeviceManagerModule interface

	virtual TSharedRef<SWidget> CreateDeviceManager(const TSharedRef<ITargetDeviceServiceManager>& DeviceServiceManager, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow) override
	{
		return SNew(SDeviceManager, DeviceServiceManager, ConstructUnderMajorTab, ConstructUnderWindow);
	}

private:

	/**
	 * Creates a new device manager tab.
	 *
	 * @param SpawnTabArgs The arguments for the tab to spawn.
	 * @return The spawned tab.
	 */
	TSharedRef<SDockTab> SpawnDeviceManagerTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.TabRole(ETabRole::MajorTab);

		DockTab->SetContent(CreateDeviceManager(TargetDeviceServiceManager.ToSharedRef(), DockTab, SpawnTabArgs.GetOwnerWindow()));

		return DockTab;
	}

private:

	// @todo gmp: implement an IoC container
	TSharedPtr<ITargetDeviceServiceManager> TargetDeviceServiceManager;

	// default widget creator for platforms that don't specify their own
	TSharedPtr<IDeviceManagerCustomPlatformWidgetCreator> DefaultWidgetCreator;

	// custom widget creators
	TMap<FString, TSharedPtr<IDeviceManagerCustomPlatformWidgetCreator>> CustomWidgetCreators;
};


IMPLEMENT_MODULE(FDeviceManagerModule, DeviceManager);
