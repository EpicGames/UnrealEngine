// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "ITargetDeviceService.h"
#include "Templates/SharedPointer.h"


/**
 * An entry in the device browser filter.
 */
struct FDeviceBrowserFilterEntry
{
	FString PlatformName;
	FName PlatformLookup;

	FDeviceBrowserFilterEntry(FString InPlatformName, FName InPlatformLookup)
		: PlatformName(InPlatformName)
		, PlatformLookup(InPlatformLookup)
	{ }
};


/**
 * Implements a filter for the device browser's target device service list.
 */
class FDeviceBrowserFilter
{
public:

	/**
	 * Filter the specified target device service based on the current filter settings.
	 *
	 * @param Device The service to filter.
	 * @return true if the service passed the filter, false otherwise.
	 */
	bool FilterDeviceService(const TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe>& DeviceService)
	{
		if (DeviceService.IsValid())
		{
			if (DisabledPlatforms.Contains(DeviceService->GetDevicePlatformDisplayName()))
			{
				return false;
			}

			if (DeviceSearchText.IsEmpty())
			{
				return true;
			}

			return (DeviceService->GetDeviceName().Contains(DeviceSearchText.ToString()));
		}

		return false;
	}

	/**
	 * Get the number of target device services that have the specified platform.
	 *
	 * @param PlatformName The name of the platform.
	 * @return The number of matching services.
	 */
	int32 GetServiceCountPerPlatform(const FString& PlatformName) const
	{
		return PlatformCounters.FindRef(PlatformName);
	}

	/**
	 * Get the device search string.
	 *
	 * @return Search string.
	 */
	const FText& GetDeviceSearchText() const
	{
		return DeviceSearchText;
	}

	/**
	 * Return the list of filtered platforms.
	 *
	 * @return List of platform names.
	 */
	const TArray<TSharedPtr<FDeviceBrowserFilterEntry>> & GetFilteredPlatforms() const
	{
		return PlatformList;
	}

	/**
	 * Check whether the specified platform is enabled in the filter.
	 *
	 * @param PlatformName The name of the platform to check.
	 * @return true if the platform is enabled, false otherwise.
	 * @see SetPlatformEnabled
	 */
	bool IsPlatformEnabled(const FString& PlatformName) const
	{
		return !DisabledPlatforms.Contains(PlatformName);
	}

	/**
	 * Populate the filter from the given list of target device proxies.
	 *
	 * @param DeviceProxies The list of device proxies to populate the filter from.
	 */
	void ResetFilter(const TArray<ITargetDeviceServicePtr>& DeviceServices)
	{
		PlatformList.Reset();
		PlatformCounters.Reset();

		for (int32 DeviceServiceIndex = 0; DeviceServiceIndex < DeviceServices.Num(); ++DeviceServiceIndex)
		{
			const ITargetDeviceServicePtr& DeviceService = DeviceServices[DeviceServiceIndex];

			// populate platforms
			const FString Platform = DeviceService->GetDevicePlatformDisplayName();
			int32& PlatformCounter = PlatformCounters.FindOrAdd(Platform);

			if (PlatformCounter == 0)
			{
				PlatformList.Add(MakeShareable(new FDeviceBrowserFilterEntry(Platform, DeviceService->GetDevicePlatformName())));
			}

			++PlatformCounter;
		}

		FilterResetEvent.Broadcast();
	}

	/**
	 * Sets the current device search string.
	 *
	 * @param SearchText The search string.
	 */
	void SetDeviceSearchString(const FText& SearchText)
	{
		if (!DeviceSearchText.EqualTo(SearchText))
		{
			DeviceSearchText = SearchText;

			FilterChangedEvent.Broadcast();
		}
	}

	/**
	 * Set the enabled state of the specified device proxy platform.
	 *
	 * @param PlatformName The name of the platform to enable or disable in the filter.
	 * @param Enabled Whether the platform is enabled.
	 * @see IsPlatformEnabled
	 */
	void SetPlatformEnabled(const FString& PlatformName, bool Enabled)
	{
		if (Enabled)
		{
			DisabledPlatforms.Remove(PlatformName);
		}
		else
		{
			DisabledPlatforms.AddUnique(PlatformName);
		}

		FilterChangedEvent.Broadcast();
	}

public:

	/**
	 * Get a delegate to be invoked when the filter state changed.
	 *
	 * @return The delegate.
	 * @see OnFilterReset
	 */
	DECLARE_EVENT(FDeviceBrowserFilter, FOnDeviceBrowserFilterChanged);
	FOnDeviceBrowserFilterChanged& OnFilterChanged()
	{
		return FilterChangedEvent;
	}

	/**
	 * Get a delegate to be invoked when the filter has been reset.
	 *
	 * @return The delegate.
	 * @see OnFilterChanged
	 */
	DECLARE_EVENT(FDeviceBrowserFilter, FOnDeviceBrowserFilterReset);
	FOnDeviceBrowserFilterReset& OnFilterReset()
	{
		return FilterResetEvent;
	}

private:

	/** The device search string. */
	FText DeviceSearchText;

	/** The list of disabled platforms. */
	TArray<FString> DisabledPlatforms;

	/** The device counters for owner filters. */
	TMap<FString, int32> OwnerCounters;

	/** The device counters for platform filters. */
	TMap<FString, int32> PlatformCounters;

	/** The list of platform filters. */
	TArray<TSharedPtr<FDeviceBrowserFilterEntry>> PlatformList;

private:

	/** An event delegate that is invoked when the filter state changed. */
	FOnDeviceBrowserFilterChanged FilterChangedEvent;

	/** An event delegate that is invoked when the filter has been reset. */
	FOnDeviceBrowserFilterReset FilterResetEvent;
};
