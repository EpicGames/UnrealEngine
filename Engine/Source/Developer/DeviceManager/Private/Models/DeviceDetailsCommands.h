// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"


#define LOCTEXT_NAMESPACE "DeviceDetailsCommands"


/**
 * The device details commands
 */
class FDeviceDetailsCommands
	: public TCommands<FDeviceDetailsCommands>
{
public:

	/** Default constructor. */
	FDeviceDetailsCommands()
		: TCommands<FDeviceDetailsCommands>(
			"DeviceDetails",
			NSLOCTEXT("Contexts", "DeviceDetails", "Device Details"),
			NAME_None, FAppStyle::GetAppStyleSetName()
		)
	{ }

public:

	//~ TCommands interface

	virtual void RegisterCommands() override
	{
		UI_COMMAND(Claim, "Claim", "Claim the device", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Release, "Release", "Release the device", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Remove, "Remove", "Remove the device", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Share, "Share", "Share the device with other users", EUserInterfaceActionType::ToggleButton, FInputChord());

		UI_COMMAND(Connect, "Connect", "Connect to the device", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Disconnect, "Disconnect", "Disconnect from the device", EUserInterfaceActionType::Button, FInputChord());

		UI_COMMAND(PowerOff, "Power Off", "Power off the device", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(PowerOffForce, "Power Off (Force)", "Power off the device (forcefully close any running applications)", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(PowerOn, "Power On", "Power on the device", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Reboot, "Reboot", "Reboot the device", EUserInterfaceActionType::Button, FInputChord());
	}

public:

	// ownership commands
	TSharedPtr<FUICommandInfo> Claim;
	TSharedPtr<FUICommandInfo> Release;
	TSharedPtr<FUICommandInfo> Remove;
	TSharedPtr<FUICommandInfo> Share;

	// connectivity commands
	TSharedPtr<FUICommandInfo> Connect;
	TSharedPtr<FUICommandInfo> Disconnect;

	// remote control commands
	TSharedPtr<FUICommandInfo> PowerOff;
	TSharedPtr<FUICommandInfo> PowerOffForce;
	TSharedPtr<FUICommandInfo> PowerOn;
	TSharedPtr<FUICommandInfo> Reboot;
};

#undef LOCTEXT_NAMESPACE
