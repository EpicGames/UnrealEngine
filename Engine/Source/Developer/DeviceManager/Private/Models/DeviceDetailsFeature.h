// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"


/**
 * Implements a view model for the device feature list.
 */
struct FDeviceDetailsFeature
{
	/** The name of the feature. */
	FString FeatureName;

	/** Whether the feature is available. */
	bool Available;

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InFeatureName The name of the feature.
	 * @param InAvailable Whether the feature is available.
	 */
	FDeviceDetailsFeature(const FString& InFeatureName, bool InAvailable)
		: FeatureName(InFeatureName)
		, Available(InAvailable)
	{ }
};
