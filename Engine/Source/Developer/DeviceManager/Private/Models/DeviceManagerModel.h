// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "ITargetDeviceService.h"
#include "Templates/SharedPointer.h"

class FDeviceManagerModel;


/**
 * Implements a view model for the messaging debugger.
 */
class FDeviceManagerModel
{
public:

	/**
	 * Get the device service that is currently selected in the device browser.
	 *
	 * @return The selected device service.
	 */
	TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe> GetSelectedDeviceService() const
	{
		return SelectedDeviceService;
	}

	/**
	 * Select the specified device service (or none if nullptr).
	 *
	 * @param DeviceService The device service to select.
	 */
	void SelectDeviceService(const TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe>& DeviceService)
	{
		if (SelectedDeviceService != DeviceService)
		{
			SelectedDeviceService = DeviceService;
			SelectedDeviceServiceChangedEvent.Broadcast();
		}
	}

public:

	/**
	 * Get an event delegate that is invoked when the selected device service has changed.
	 *
	 * @return The event delegate.
	 */
	DECLARE_EVENT(FDeviceManagerModel, FOnSelectedDeviceServiceChanged);
	FOnSelectedDeviceServiceChanged& OnSelectedDeviceServiceChanged()
	{
		return SelectedDeviceServiceChangedEvent;
	}

private:

	/** The currently selected target device service. */
	TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe> SelectedDeviceService;

	/** An event delegate that is invoked when the selected message has changed. */
	FOnSelectedDeviceServiceChanged SelectedDeviceServiceChangedEvent;
};
