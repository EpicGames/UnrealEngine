// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Apps/SDeviceApps.h"

#include "Styling/AppStyle.h"
#include "SlateOptMacros.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#include "Models/DeviceManagerModel.h"
#include "Widgets/Apps/SDeviceAppsAppListRow.h"


#define LOCTEXT_NAMESPACE "SDeviceApps"


/* SMessagingEndpoints structors
*****************************************************************************/

SDeviceApps::~SDeviceApps()
{
	if (Model.IsValid())
	{
		Model->OnSelectedDeviceServiceChanged().RemoveAll(this);
	}
}


/* SDeviceDetails interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDeviceApps::Construct(const FArguments& InArgs, const TSharedRef<FDeviceManagerModel>& InModel)
{
	Model = InModel;

	// callback for getting the enabled state of the processes panel
	auto AppsBoxIsEnabled = [this]() -> bool {
		ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();
		return (DeviceService.IsValid() && DeviceService->CanStart());
	};

	// callback for generating a row widget in the application list view
	auto AppListViewGenerateRow = [this](TSharedPtr<FString> Item, const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow> {
		return SNew(SDeviceAppsAppListRow, OwnerTable);
	};

	// callback for selecting items in the devices list view
	auto AppListViewSelectionChanged = [this](TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo) {
		// @todo
	};

	// callback for getting the visibility of the 'Select a device' message. */
	auto SelectDeviceOverlayVisibility = [this]() -> EVisibility {
		if (Model->GetSelectedDeviceService().IsValid())
		{
			return EVisibility::Hidden;
		}

		return EVisibility::Visible;
	};

	// construct children
	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
			[
				SNew(SVerticalBox)
					.IsEnabled_Lambda(AppsBoxIsEnabled)

				+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(0.0f, 4.0f, 0.0f, 0.0f)
					[
						// applications list
						SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							.Padding(0.0f)
							[
								SAssignNew(AppListView, SListView<TSharedPtr<FString> >)
									.ListItemsSource(&AppList)
									.OnGenerateRow_Lambda(AppListViewGenerateRow)
									.OnSelectionChanged_Lambda(AppListViewSelectionChanged)
									.SelectionMode(ESelectionMode::Single)
									.HeaderRow
									(
										SNew(SHeaderRow)

										+ SHeaderRow::Column("Name")
											.DefaultLabel(LOCTEXT("AppListNameColumnHeader", "Name"))

										+ SHeaderRow::Column("Date")
											.DefaultLabel(LOCTEXT("AppListDeploymentDateColumnHeader", "Date deployed"))

										+ SHeaderRow::Column("Owner")
											.DefaultLabel(LOCTEXT("AppListOwnerColumnHeader", "Deployed by"))
									)
							]
					]
			]

		+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NotificationList.ItemBackground"))
					.Padding(8.0f)
					.Visibility_Lambda(SelectDeviceOverlayVisibility)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("SelectSessionOverlayText", "Please select a device from the Device Browser"))
					]
			]
	];

	// callback for handling device service selection changes
	auto ModelSelectedDeviceServiceChanged = [this]() {
		// @todo
	};

	Model->OnSelectedDeviceServiceChanged().AddLambda(ModelSelectedDeviceServiceChanged);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
