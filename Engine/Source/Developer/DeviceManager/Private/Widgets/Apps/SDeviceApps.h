// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FDeviceManagerModel;


/**
 * Implements the device details widget.
 */
class SDeviceApps
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceApps) { }
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SDeviceApps();

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InModel The view model to use.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FDeviceManagerModel>& InModel);
	
private:

	/** The list of applications deployed to the device. */
	TArray<TSharedPtr<FString>> AppList;

	/** The application list view. */
	TSharedPtr<SListView<TSharedPtr<FString>>> AppListView;

	/** Pointer the device manager's view model. */
	TSharedPtr<FDeviceManagerModel> Model;
};
