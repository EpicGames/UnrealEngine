// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Layout/Margin.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"


#define LOCTEXT_NAMESPACE "SDeviceAppsAppListRow"


/**
 * Implements a row widget for the application list view.
 */
class SDeviceAppsAppListRow
	: public SMultiColumnTableRow<TSharedPtr<FString>>
{
public:

	SLATE_BEGIN_ARGS(SDeviceAppsAppListRow) { }
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		SMultiColumnTableRow<TSharedPtr<FString> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName The name of the column to generate the widget for.
	 * @return The widget.
	 */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == TEXT("Date"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("TempDateColumn", "@todo gmp: implement"))
				];
		}
		else if (ColumnName == TEXT("Name"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("TempNameColumn", "@todo gmp: implement"))
				];
		}
		else if (ColumnName == TEXT("Owner"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("TempOwnerColumn", "@todo gmp: implement"))
				];
		}

		return SNullWidget::NullWidget;
	}
};


#undef LOCTEXT_NAMESPACE
