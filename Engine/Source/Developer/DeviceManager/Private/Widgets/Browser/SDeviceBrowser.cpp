// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Browser/SDeviceBrowser.h"

#include "Styling/AppStyle.h"
#include "Framework/Commands/UICommandList.h"
#include "ITargetDeviceServiceManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#include "Widgets/Browser/SDeviceBrowserContextMenu.h"
#include "Widgets/Browser/SDeviceBrowserDeviceAdder.h"
#include "Widgets/Browser/SDeviceBrowserDeviceListRow.h"
#include "Widgets/Browser/SDeviceBrowserFilterBar.h"
#include "Widgets/Browser/SDeviceBrowserTooltip.h"
#include "Models/DeviceBrowserFilter.h"
#include "Models/DeviceManagerModel.h"


#define LOCTEXT_NAMESPACE "SDeviceBrowser"


/* SDeviceBrowser interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDeviceBrowser::Construct(const FArguments& InArgs, const TSharedRef<FDeviceManagerModel>& InModel, const TSharedRef<ITargetDeviceServiceManager>& InDeviceServiceManager, const TSharedPtr<FUICommandList>& InUICommandList)
{
	DeviceServiceManager = InDeviceServiceManager;
	Filter = MakeShareable(new FDeviceBrowserFilter());
	Model = InModel;
	NeedsServiceListRefresh = true;
	UICommandList = InUICommandList;

	auto DeviceServiceListViewContextMenuOpening = [this]() -> TSharedPtr<SWidget> {
		TArray<TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe>> SelectedDeviceServices = DeviceServiceListView->GetSelectedItems();

		if (SelectedDeviceServices.Num() > 0)
		{
			return SNew(SDeviceBrowserContextMenu, UICommandList);
		}

		return nullptr;
	};

	auto DeviceServiceListViewHighlightText = [this]() -> FText {
		return Filter->GetDeviceSearchText();
	};

	auto DeviceServiceListViewGenerateRow = [=](TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe> DeviceService, const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow> {
		return SNew(SDeviceBrowserDeviceListRow, OwnerTable)
			.DeviceService(DeviceService)
			.HighlightText_Lambda(DeviceServiceListViewHighlightText)
			.ToolTip(SNew(SDeviceBrowserTooltip, DeviceService.ToSharedRef()));
	};

	auto DeviceServiceListViewSelectionChanged = [this](TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe> Selection, ESelectInfo::Type SelectInfo) {
		Model->SelectDeviceService(Selection);
	};

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				/*
				SNew(SExpandableArea)
					.AreaTitle(LOCTEXT("FilterBarAreaTitle", "Device Filter").ToString())
					.InitiallyCollapsed(true)
					.Padding(FMargin(8.0f, 8.0f, 8.0f, 4.0f))
					.BodyContent()
					[*/
						// filter bar
						SNew(SDeviceBrowserFilterBar, Filter.ToSharedRef())
					//]
			]

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 4.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Padding(0.0f)
					[
						// device list
						SAssignNew(DeviceServiceListView, SListView<ITargetDeviceServicePtr>)
							.ListItemsSource(&DeviceServiceList)
							.OnContextMenuOpening_Lambda(DeviceServiceListViewContextMenuOpening)
							.OnGenerateRow_Lambda(DeviceServiceListViewGenerateRow)
							.OnSelectionChanged_Lambda(DeviceServiceListViewSelectionChanged)
							.SelectionMode(ESelectionMode::Single)
							.HeaderRow
							(
								SNew(SHeaderRow)

								+ SHeaderRow::Column("Icon")
									.DefaultLabel(FText::FromString(TEXT(" ")))
									.FixedWidth(32.0f)

								+ SHeaderRow::Column("Name")
									.DefaultLabel(LOCTEXT("DevicesListNameColumnHeader", "Device"))

								+ SHeaderRow::Column("Platform")
									.DefaultLabel(LOCTEXT("DevicesListPlatformColumnHeader", "Platform"))

								+ SHeaderRow::Column("Status")
									.DefaultLabel(LOCTEXT("DevicesListStatusColumnHeader", "Status"))
									.FixedWidth(128.0f)

								+ SHeaderRow::Column("Claimed")
									.DefaultLabel(LOCTEXT("DevicesListClaimedColumnHeader", "Claimed By"))

								+ SHeaderRow::Column("Share")
									.DefaultLabel(LOCTEXT("DevicesListShareColumnHeader", "Share"))
									.FixedWidth(48.0f)
									.HAlignCell(HAlign_Center)
									.HAlignHeader(HAlign_Center)
							)
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 4.0f, 0.0f, 0.0f)
			[
				SNew(SExpandableArea)
					.AreaTitle(LOCTEXT("DeviceAdderAreaTitle", "Add An Unlisted Device"))
					.InitiallyCollapsed(true)
					.Padding(FMargin(8.0f, 8.0f, 8.0f, 4.0f))
					.BodyContent()
					[
						// device adder
						SNew(SDeviceBrowserDeviceAdder, InDeviceServiceManager)
					]
			]
	];

	DeviceServiceManager->OnServiceAdded().AddLambda([this](const TSharedRef<ITargetDeviceService, ESPMode::ThreadSafe>& AddedService) { NeedsServiceListRefresh = true; });
	DeviceServiceManager->OnServiceRemoved().AddLambda([this](const TSharedRef<ITargetDeviceService, ESPMode::ThreadSafe>& RemovedService) { NeedsServiceListRefresh = true; });
	
	Filter->OnFilterChanged().AddLambda([this]() { ReloadDeviceServiceList(false); });
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SDeviceBrowser::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	//@TODO Passive - Only happens in response to the addition or removal of a device to the device service manager
	if (NeedsServiceListRefresh)
	{
		ReloadDeviceServiceList(true);
		NeedsServiceListRefresh = false;
	}
}


/* SDeviceBrowser callbacks
 *****************************************************************************/

void SDeviceBrowser::ReloadDeviceServiceList(bool FullyReload)
{
	// reload target device service list
	if (FullyReload)
	{
		AvailableDeviceServices.Reset();

		DeviceServiceManager->GetServices(AvailableDeviceServices);
		Filter->ResetFilter(AvailableDeviceServices);
	}

	// filter target device service list
	DeviceServiceList.Reset();

	for (int32 DeviceServiceIndex = 0; DeviceServiceIndex < AvailableDeviceServices.Num(); ++DeviceServiceIndex)
	{
		const TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe>& DeviceService = AvailableDeviceServices[DeviceServiceIndex];

		if (Filter->FilterDeviceService(DeviceService.ToSharedRef()))
		{
			DeviceServiceList.Add(DeviceService);
		}
	}

	// refresh list view
	DeviceServiceListView->RequestListRefresh();
}


#undef LOCTEXT_NAMESPACE
