// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "ITargetDeviceService.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FDeviceBrowserFilter;
class FDeviceManagerModel;
class FUICommandList;
class ITargetDeviceServiceManager;


/**
 * Delegate type for selection changes in the device browser.
 *
 * The first parameter is the newly selected device service (or nullptr if none was selected).
 */
DECLARE_DELEGATE_OneParam(FOnDeviceBrowserSelectionChanged, const ITargetDeviceServicePtr&)


/**
 * Implements the device browser widget.
 */
class SDeviceBrowser
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceBrowser) { }
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InModel The view model to use.
	 * @param InDeviceServiceManager The target device service manager to use.
	 * @param InUICommandList The UI command list to use.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FDeviceManagerModel>& InModel, const TSharedRef<ITargetDeviceServiceManager>& InDeviceServiceManager, const TSharedPtr<FUICommandList>& InUICommandList);

public:

	//~ SCompoundWidget overrides

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	/**
	 * Reloads the list of target device services.
	 *
	 * @param FullyReload Whether to fully reload the service entries, or only re-apply filtering.
	 */
	void ReloadDeviceServiceList(bool FullyReload);

private:

	/** Holds the list of all available target device services. */
	TArray<TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe>> AvailableDeviceServices;

	/** Holds the list of filtered target device services for display. */
	TArray<TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe>> DeviceServiceList;

	/** Holds the list view for filtered target device services. */
	TSharedPtr<SListView<ITargetDeviceServicePtr>> DeviceServiceListView;

	/** Holds a pointer to the target device service manager. */
	TSharedPtr<ITargetDeviceServiceManager> DeviceServiceManager;

	/** Holds the filter model. */
	TSharedPtr<FDeviceBrowserFilter> Filter;

	/** Holds a pointer the device manager's view model. */
	TSharedPtr<FDeviceManagerModel> Model;

	/** Holds a flag indicating whether the list of target device services needs to be refreshed. */
	bool NeedsServiceListRefresh;

	/** Holds a pointer to the command list for controlling the device. */
	TSharedPtr<FUICommandList> UICommandList;
};
