// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBorder.h"

#include "Models/DeviceDetailsCommands.h"


#define LOCTEXT_NAMESPACE "SDeviceBrowserContextMenu"


/**
 * Implements a context menu for the device browser list view.
 */
class SDeviceBrowserContextMenu
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceBrowserContextMenu) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InUICommandList The UI command list to use.
	 */
	void Construct(const FArguments& InArgs, const TSharedPtr<FUICommandList>& InUICommandList)
	{
		ChildSlot
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Content()
				[
					MakeContextMenu(InUICommandList)
				]
		];
	}

protected:

	/**
	 * Build the context menu widget.
	 *
	 * @return The context menu.
	 */
	TSharedRef<SWidget> MakeContextMenu(const TSharedPtr<FUICommandList>& CommandList)
	{
		FMenuBuilder MenuBuilder(true, CommandList);

		// ownership actions
		MenuBuilder.BeginSection("Ownership", LOCTEXT("OwnershipMenuHeading", "Ownership"));
		{
			MenuBuilder.AddMenuEntry(FDeviceDetailsCommands::Get().Claim);
			MenuBuilder.AddMenuEntry(FDeviceDetailsCommands::Get().Release);
			MenuBuilder.AddMenuEntry(FDeviceDetailsCommands::Get().Share);
			MenuBuilder.AddMenuEntry(FDeviceDetailsCommands::Get().Remove);
		}
		MenuBuilder.EndSection();

		// connectivity actions
		MenuBuilder.BeginSection("Connectivity", LOCTEXT("ConnectivityMenuHeading", "Connectivity"));
		{
			MenuBuilder.AddMenuEntry(FDeviceDetailsCommands::Get().Connect);
			MenuBuilder.AddMenuEntry(FDeviceDetailsCommands::Get().Disconnect);
		}
		MenuBuilder.EndSection();

		// remote control actions
		MenuBuilder.BeginSection("RemoteControl", LOCTEXT("RemoteControlMenuHeading", "Remote Control"));
		{
			MenuBuilder.AddMenuEntry(FDeviceDetailsCommands::Get().PowerOn);
			MenuBuilder.AddMenuEntry(FDeviceDetailsCommands::Get().PowerOff);
			MenuBuilder.AddMenuEntry(FDeviceDetailsCommands::Get().PowerOffForce);
			MenuBuilder.AddMenuEntry(FDeviceDetailsCommands::Get().Reboot);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}
};


#undef LOCTEXT_NAMESPACE
