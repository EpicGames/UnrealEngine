// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Browser/SDeviceBrowserDeviceAdder.h"

#include "Internationalization/Text.h"
#include "ITargetDeviceServiceManager.h"
#include "Misc/MessageDialog.h"
#include "Misc/CoreMisc.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Styling/AppStyle.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/IDeviceManagerCustomPlatformWidgetCreator.h"
#include "PlatformInfo.h"
#include "IDeviceManagerModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "SDeviceBrowserDeviceAdder"


/* SDeviceBrowserDeviceAdder interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDeviceBrowserDeviceAdder::Construct(const FArguments& InArgs, const TSharedRef<ITargetDeviceServiceManager>& InDeviceServiceManager)
{
	DeviceServiceManager = InDeviceServiceManager;

	// callback for clicking of the Add button
	auto AddButtonClicked = [this]() -> FReply {
		ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(*PlatformComboBox->GetSelectedItem());
		check(TargetPlatform);

		TargetPlatform->GetCustomWidgetCreator()->AddDevice(TargetPlatform->PlatformName(), CustomPlatformWidget);

		return FReply::Handled();
	};

	// callback for determining the enabled state of the 'Add' button
	auto AddButtonIsEnabled = [this]() -> bool {
		TSharedPtr<FString> PlatformName = PlatformComboBox->GetSelectedItem();

		if (PlatformName.IsValid())
		{
			ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(*PlatformComboBox->GetSelectedItem());
			check(TargetPlatform);
			return TargetPlatform->GetCustomWidgetCreator()->IsAddDeviceInputValid(*PlatformName, CustomPlatformWidget);
		}

		return false;
	};

	// callback for getting the name of the selected platform
	auto PlatformComboBoxContentText = [this]() -> FText {
		TSharedPtr<FString> SelectedPlatform = PlatformComboBox->GetSelectedItem();
		return SelectedPlatform.IsValid() ? FText::FromString(*SelectedPlatform) : LOCTEXT("SelectAPlatform", "Select a Platform");
	};

	// callback for generating widgets for the platforms combo box
	auto PlatformComboBoxGenerateWidget = [this](TSharedPtr<FString> Item) -> TSharedRef<SWidget> {
		const PlatformInfo::FTargetPlatformInfo* const PlatformInfo = PlatformInfo::FindPlatformInfo(**Item);

		return
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				[
					SNew(SBox)
						.WidthOverride(24.f)
						.HeightOverride(24.f)
						[
							SNew(SImage)
								.Image((PlatformInfo) ? FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Normal)) : FStyleDefaults::GetNoBrush())
						]
				]

			+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(FText::FromString(*Item))
				];
	};

	// callback for handling platform selection changes
	auto PlatformComboBoxSelectionChanged = [this](TSharedPtr<FString> StringItem, ESelectInfo::Type SelectInfo) {
		
		FString PlatformName = *StringItem;

		ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(PlatformName);
		check(TargetPlatform);

		// create custom widget for the platform and place it in the panel
		CustomPlatformWidget = TargetPlatform->GetCustomWidgetCreator()->CreateAddDeviceWidget(PlatformName);
		CustomPlatformWidgetPanel->SetContent(CustomPlatformWidget.ToSharedRef());
	};

	// construct children
	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)

				// platform selector
				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Left)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("PlatformLabel", "Platform:"))
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Left)
							.Padding(0.0f, 4.0f, 0.0f, 0.0f)
							[
								SAssignNew(PlatformComboBox, SComboBox<TSharedPtr<FString>>)
									.ContentPadding(FMargin(6.0f, 2.0f))
									.OptionsSource(&PlatformList)
									.OnGenerateWidget_Lambda(PlatformComboBoxGenerateWidget)
									.OnSelectionChanged_Lambda(PlatformComboBoxSelectionChanged)
									[
										SNew(STextBlock)
											.Text_Lambda(PlatformComboBoxContentText)
									]
							]
					]

				// custom platform widget
				+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SAssignNew(CustomPlatformWidgetPanel, SBox)
					]

				// add button
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Bottom)
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SAssignNew(AddButton, SButton)
							.ContentPadding(FMargin(9.0, 2.0))
							.IsEnabled_Lambda(AddButtonIsEnabled)
							.Text(LOCTEXT("AddButtonText", "Add"))
							.OnClicked_Lambda(AddButtonClicked)
					]
			]
	];

	RefreshPlatformList();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SDeviceBrowserDeviceAdder implementation
 *****************************************************************************/

void SDeviceBrowserDeviceAdder::RefreshPlatformList()
{
	TArray<ITargetPlatform*> Platforms = GetTargetPlatformManager()->GetTargetPlatforms();

	PlatformList.Reset();

	for (int32 Index = 0; Index < Platforms.Num(); ++Index)
	{
		PlatformList.Add(MakeShareable(new FString(Platforms[Index]->PlatformName())));
	}

	PlatformComboBox->RefreshOptions();
}

#undef LOCTEXT_NAMESPACE
