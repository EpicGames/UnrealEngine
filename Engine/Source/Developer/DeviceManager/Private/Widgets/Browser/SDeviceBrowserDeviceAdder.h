// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"

class FString;
class ITargetDeviceServiceManager;
class SButton;
class SEditableTextBox;
class SOverlay;

/**
 * Implements a widget for manually locating target devices.
 */
class SDeviceBrowserDeviceAdder
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceBrowserDeviceAdder) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InDeviceManager The target device manager to use.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<ITargetDeviceServiceManager>& InDeviceServiceManager);

protected:

	/** Refreshes the list of known platforms. */
	void RefreshPlatformList();

private:

	/** The button for adding an unlisted device. */
	TSharedPtr<SButton> AddButton;

	/** Panel on which custom widget will be placed */
	TSharedPtr<SBox> CustomPlatformWidgetPanel;
	
	/** Platform customizable widget */
	TSharedPtr<SWidget> CustomPlatformWidget;

	/** Holds a pointer to the target device service manager. */
	TSharedPtr<ITargetDeviceServiceManager> DeviceServiceManager;

	/** The platforms combo box. */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> PlatformComboBox;

	/** The list of known platforms. */
	TArray<TSharedPtr<FString>> PlatformList;
};
