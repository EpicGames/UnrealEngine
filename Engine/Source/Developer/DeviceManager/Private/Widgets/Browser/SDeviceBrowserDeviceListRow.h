// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Styling/AppStyle.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/Text.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceService.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "PlatformInfo.h"
#include "SlateOptMacros.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleDefaults.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"


#define LOCTEXT_NAMESPACE "SDeviceBrowserDeviceListRow"


/**
 * Implements a row widget for the device list view.
 */
class SDeviceBrowserDeviceListRow
	: public SMultiColumnTableRow<TSharedPtr<ITargetDeviceProxy>>
{
public:

	SLATE_BEGIN_ARGS(SDeviceBrowserDeviceListRow) { }
		SLATE_ARGUMENT(ITargetDeviceServicePtr, DeviceService)
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

public:

	/**
	 * Construct the widget.
	 *
	 * @param InArgs The arguments.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		check(InArgs._DeviceService.IsValid());

		DeviceService = InArgs._DeviceService;
		HighlightText = InArgs._HighlightText;

		
		SMultiColumnTableRow<TSharedPtr<ITargetDeviceProxy>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	//~ SMultiColumnTableRow interface

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == TEXT("Claimed"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SDeviceBrowserDeviceListRow::HandleTextColorAndOpacity)
						.Text(this, &SDeviceBrowserDeviceListRow::HandleClaimedText)					
				];
		}
		else if (ColumnName == TEXT("Icon"))
		{
			const PlatformInfo::FTargetPlatformInfo* const PlatformInfo = PlatformInfo::FindPlatformInfo(DeviceService->GetDevicePlatformName());

			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.WidthOverride(24.f)
				.HeightOverride(24.f)
				[
					SNew(SImage)
						.Image((PlatformInfo) ? FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Normal)) : FStyleDefaults::GetNoBrush())
				];
		}
		else if (ColumnName == TEXT("Name"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SDeviceBrowserDeviceListRow::HandleTextColorAndOpacity)
						.HighlightText(HighlightText)
						.Text(this, &SDeviceBrowserDeviceListRow::HandleNameText)
				];
		}
		else if (ColumnName == TEXT("Platform"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SDeviceBrowserDeviceListRow::HandleTextColorAndOpacity)
						.Text(FText::FromString(DeviceService->GetDevicePlatformDisplayName()))
				];
		}
		else if (ColumnName == TEXT("Share"))
		{
			return SNew(SBox)
				.Padding(2.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
						.IsChecked(this, &SDeviceBrowserDeviceListRow::HandleShareCheckBoxIsChecked)
						.IsEnabled(this, &SDeviceBrowserDeviceListRow::HandleShareCheckBoxIsEnabled)
						.OnCheckStateChanged(this, &SDeviceBrowserDeviceListRow::HandleShareCheckBoxStateChanged)
						.ToolTipText(LOCTEXT("ShareCheckBoxToolTip", "Check this box to share this device with other users on the network"))
				];
		}
		else if (ColumnName == TEXT("Status"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SDeviceBrowserDeviceListRow::HandleTextColorAndOpacity)
						.Text(this, &SDeviceBrowserDeviceListRow::HandleStatusTextBlockText)
				];
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	// Callback for getting the text in the 'Claimed' column.
	FText HandleClaimedText() const
	{
		FString ClaimUser = DeviceService->GetClaimUser();

		if ((ClaimUser == FPlatformProcess::UserName(false)) && !DeviceService->IsRunning())
		{
			ClaimUser += LOCTEXT("RemotelyHint", " (remotely)").ToString();
		}

		return FText::FromString(ClaimUser);
	}

	// Callback for getting the text in the 'Name' column.
	FText HandleNameText() const
	{
		return FText::FromString(DeviceService->GetDeviceName());
	}

	// Callback for changing this row's Share check box state.
	void HandleShareCheckBoxStateChanged(ECheckBoxState NewState)
	{
		DeviceService->SetShared(NewState == ECheckBoxState::Checked);
	}

	// Callback for getting the state of the 'Share' check box.
	ECheckBoxState HandleShareCheckBoxIsChecked() const
	{
		if (DeviceService->IsShared())
		{
			return ECheckBoxState::Checked;
		}

		return ECheckBoxState::Unchecked;
	}

	// Callback for getting the enabled state of the 'Share' check box.
	bool HandleShareCheckBoxIsEnabled() const
	{
		return DeviceService->IsRunning();
	}

	// Callback for getting the status text.
	FText HandleStatusTextBlockText() const
	{
		ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

		if (TargetDevice.IsValid())
		{
			if (TargetDevice->IsConnected())
			{
				return LOCTEXT("StatusConnected", "Connected");
			}

			return LOCTEXT("StatusDisconnected", "Disconnected");
		}

		return LOCTEXT("StatusUnavailable", "Unavailable");
	}

	// Callback for getting the text color.
	FSlateColor HandleTextColorAndOpacity() const
	{
		if (DeviceService->CanStart())
		{
			return FSlateColor::UseForeground();
		}

		return FSlateColor::UseSubduedForeground();
	}

private:

	// Holds the target device service used to populate this row.
	TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe> DeviceService;

	// Holds the highlight text for the log message.
	TAttribute<FText> HighlightText;
};


#undef LOCTEXT_NAMESPACE
