// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Browser/SDeviceBrowserFilterBar.h"

#include "Styling/AppStyle.h"
#include "PlatformInfo.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/STextBlock.h"

#include "Models/DeviceBrowserFilter.h"


#define LOCTEXT_NAMESPACE "SDeviceBrowserFilterBar"


/* SSessionBrowserFilterBar structors
 *****************************************************************************/

SDeviceBrowserFilterBar::~SDeviceBrowserFilterBar()
{
	if (Filter.IsValid())
	{
		Filter->OnFilterReset().RemoveAll(this);
	}	
}


/* SSessionBrowserFilterBar interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDeviceBrowserFilterBar::Construct(const FArguments& InArgs, TSharedRef<FDeviceBrowserFilter> InFilter)
{
	Filter = InFilter;

	// callback for filter model resets
	auto FilterReset = [this]() {
		FilterStringTextBox->SetText(Filter->GetDeviceSearchText());
		PlatformListView->RequestListRefresh();
	};

	// callback for changing the filter string text box text
	auto FilterStringTextChanged = [this](const FText& NewText) {
		Filter->SetDeviceSearchString(NewText);
	};

	// callback for generating a row widget for the platform filter list
	auto PlatformListViewGenerateRow = [this](TSharedPtr<FDeviceBrowserFilterEntry> PlatformEntry, const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow> {
		const PlatformInfo::FTargetPlatformInfo* const PlatformInfo = PlatformInfo::FindPlatformInfo(PlatformEntry->PlatformLookup);

		return SNew(STableRow<TSharedPtr<FString> >, OwnerTable)
			.Content()
			[
				SNew(SCheckBox)
				.IsChecked_Lambda(
					[this, PlatformEntry]() -> ECheckBoxState {
						return Filter->IsPlatformEnabled(PlatformEntry->PlatformName)
							? ECheckBoxState::Checked
							: ECheckBoxState::Unchecked;
					}
				)
				.Padding(FMargin(6.0, 2.0))
				.OnCheckStateChanged_Lambda(
					[this, PlatformEntry](ECheckBoxState CheckState) {
						Filter->SetPlatformEnabled(PlatformEntry->PlatformName, CheckState == ECheckBoxState::Checked);
					}
				)
				.Content()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SBox)
								.WidthOverride(24.f)
								.HeightOverride(24.f)
								[
									SNew(SImage)
									.Image((PlatformInfo) ? FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Normal)) : FStyleDefaults::GetNoBrush())
								]
						]

					+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
								.Text_Lambda(
									[this, PlatformEntry]() -> FText {
										return FText::Format(LOCTEXT("PlatformListRowFmt", "{0} ({1})"), FText::FromString(PlatformEntry->PlatformName), FText::AsNumber(Filter->GetServiceCountPerPlatform(PlatformEntry->PlatformName)));
									}
								)
						]
				]
			];
	};

	// construct children
	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.Padding(6.0f, 0.0f, 6.0f, 0.0f)
			[
				// platform filter
				SNew(SComboButton)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("NormalBold"))
					.Text(LOCTEXT("PlatformFiltersComboButtonText", "Platform Filters"))
				]
				.ContentPadding(0.0f)
				.MenuContent()
				[
					SAssignNew(PlatformListView, SListView<TSharedPtr<FDeviceBrowserFilterEntry> >)
					.ListItemsSource(&Filter->GetFilteredPlatforms())
					.OnGenerateRow_Lambda(PlatformListViewGenerateRow)
				]
			]

		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Top)
			[
				// search box
				SAssignNew(FilterStringTextBox, SSearchBox)
					.HintText(LOCTEXT("SearchBoxHint", "Search devices"))
					.OnTextChanged_Lambda(FilterStringTextChanged)
			]

	];

	Filter->OnFilterReset().AddLambda(FilterReset);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
