// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FDeviceBrowserFilter;
class SSearchBox;

struct FDeviceBrowserFilterEntry;


/**
 * Implements the device browser filter bar widget.
 */
class SDeviceBrowserFilterBar
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceBrowserFilterBar) { }
	SLATE_END_ARGS()

public:

	/** Default constructor. */
	SDeviceBrowserFilterBar()
		: Filter(nullptr)
	{ }

	/** Destructor. */
	~SDeviceBrowserFilterBar();

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InFilter The filter model to use.
	 */
	void Construct(const FArguments& InArgs, TSharedRef<FDeviceBrowserFilter> InFilter);

private:

	/** Pointer to the filter model. */
	TSharedPtr<FDeviceBrowserFilter> Filter;

	/** The filter string text box. */
	TSharedPtr<SSearchBox> FilterStringTextBox;

	/** The platform filters list view. */
	TSharedPtr<SListView<TSharedPtr<FDeviceBrowserFilterEntry>>> PlatformListView;
};
