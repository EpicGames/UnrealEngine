// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SToolTip.h"

#include "Widgets/Shared/SDeviceQuickInfo.h"

class ITargetDeviceService;


#define LOCTEXT_NAMESPACE "SDeviceBrowserTooltip"


/**
 * Implements a tool tip for widget the device browser.
 */
class SDeviceBrowserTooltip
	: public SToolTip
{
public:

	SLATE_BEGIN_ARGS(SDeviceBrowserTooltip) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InDeviceServiceManager The target device service manager to use.
	 * @param InDeviceManagerState The optional device manager view state.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<ITargetDeviceService, ESPMode::ThreadSafe>& InDeviceService)
	{
		SToolTip::Construct(
			SToolTip::FArguments()
				.Content()
				[
					SNew(SDeviceQuickInfo)
						.InitialDeviceService(InDeviceService)
				]
		);
	}
};


#undef LOCTEXT_NAMESPACE
