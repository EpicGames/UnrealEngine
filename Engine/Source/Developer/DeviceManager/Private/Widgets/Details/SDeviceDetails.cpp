// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Details/SDeviceDetails.h"
#include "Widgets/SOverlay.h"
#include "SlateOptMacros.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/IDeviceManagerCustomPlatformWidgetCreator.h"
#include "Widgets/Shared/SDeviceQuickInfo.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

#include "Models/DeviceDetailsFeature.h"
#include "Models/DeviceManagerModel.h"
#include "Widgets/Details/SDeviceDetailsFeatureListRow.h"


#define LOCTEXT_NAMESPACE "SDeviceDetails"


/* SMessagingEndpoints structors
*****************************************************************************/

SDeviceDetails::~SDeviceDetails()
{
	if (Model.IsValid())
	{
		Model->OnSelectedDeviceServiceChanged().RemoveAll(this);
	}
}


/* SDeviceDetails interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDeviceDetails::Construct(const FArguments& InArgs, const TSharedRef<FDeviceManagerModel>& InModel)
{
	Model = InModel;

	// callback for getting the visibility of the details box.
	auto DetailsBoxVisibility = [this]() -> EVisibility {
		return Model->GetSelectedDeviceService().IsValid()
			? EVisibility::Visible
			: EVisibility::Hidden;
	};

	// callback for generating a row widget for the feature list view.
	auto FeatureListGenerateRow = [](TSharedPtr<FDeviceDetailsFeature> Feature, const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow> {
		return SNew(SDeviceDetailsFeatureListRow, OwnerTable, Feature.ToSharedRef());
			//.Style(Style);
	};

	// callback for getting the visibility of the 'Select a device' message.
	auto HandleSelectDeviceOverlayVisibility = [this]() -> EVisibility {
		return Model->GetSelectedDeviceService().IsValid()
			? EVisibility::Hidden
			: EVisibility::Visible;
	};

	// construct children
	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
			[
				SNew(SVerticalBox)
					.Visibility_Lambda(DetailsBoxVisibility)

				+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.0f, 2.0f)
					[
						// quick info
						SAssignNew(QuickInfo, SDeviceQuickInfo)
					]

				// custom platform widget
				+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SAssignNew(CustomPlatformWidgetPanel, SBox)
					]

				+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(0.0f, 8.0f, 0.0f, 0.0f)
					[
						SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							.Padding(0.0f)
							[
								// feature list view
								SAssignNew(FeatureListView, SListView<TSharedPtr<FDeviceDetailsFeature>>)
									.ListItemsSource(&FeatureList)
									.OnGenerateRow_Lambda(FeatureListGenerateRow)
									.SelectionMode(ESelectionMode::None)
									.HeaderRow
									(
										SNew(SHeaderRow)

										+ SHeaderRow::Column("Feature")
											.DefaultLabel(LOCTEXT("FeatureListFeatureColumnHeader", "Feature"))
											.FillWidth(0.6f)

										+ SHeaderRow::Column("Available")
											.DefaultLabel(LOCTEXT("FeatureListAvailableColumnHeader", "Available"))
											.FillWidth(0.4f)
									)
							]
					]
			]

		+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NotificationList.ItemBackground"))
					.Padding(8.0f)
					.Visibility_Lambda(HandleSelectDeviceOverlayVisibility)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("SelectSessionOverlayText", "Please select a device from the Device Browser"))
					]
			]
	];

	// callback for handling device service selection changes.
	auto ModelSelectedDeviceServiceChanged = [this]() {
		FeatureList.Empty();

		ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

		if (DeviceService.IsValid())
		{
			ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

			if (TargetDevice.IsValid())
			{
				const ITargetPlatformSettings& TargetPlatformSettings = TargetDevice->GetPlatformSettings();
				const ITargetPlatformControls& TargetPlatformControls = TargetDevice->GetPlatformControls();

				// platform features
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("AudioStreaming"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::AudioStreaming))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("DistanceFieldShadows"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::DistanceFieldShadows))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("DistanceFieldAO"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::DistanceFieldAO))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("GrayscaleSRGB"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::GrayscaleSRGB))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("HighQualityLightmaps"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::HighQualityLightmaps))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("LowQualityLightmaps"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::LowQualityLightmaps))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("MultipleGameInstances"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::MultipleGameInstances))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("Packaging"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::Packaging))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("SdkConnectDisconnect"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::SdkConnectDisconnect))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("TextureStreaming"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::TextureStreaming))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("UserCredentials"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::UserCredentials))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("MobileRendering"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::MobileRendering))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("DeferredRendering"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::DeferredRendering))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("LumenGI"), TargetPlatformSettings.SupportsFeature(ETargetPlatformFeatures::LumenGI))));

				// device features
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("MultiLaunch"), TargetDevice->SupportsFeature(ETargetDeviceFeatures::MultiLaunch))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("PowerOff"), TargetDevice->SupportsFeature(ETargetDeviceFeatures::PowerOff))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("PowerOn"), TargetDevice->SupportsFeature(ETargetDeviceFeatures::PowerOn))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("ProcessSnapshot"), TargetDevice->SupportsFeature(ETargetDeviceFeatures::ProcessSnapshot))));
				FeatureList.Add(MakeShareable(new FDeviceDetailsFeature(TEXT("Reboot"), TargetDevice->SupportsFeature(ETargetDeviceFeatures::Reboot))));

				// create custom widget for the platform and place it in the panel
				TSharedPtr<SWidget> CustomWidget = TargetPlatformControls.GetCustomWidgetCreator()->CreateDeviceInfoWidget(TargetPlatformControls.PlatformName(), TargetDevice);
				if (CustomWidget)
				{
					CustomPlatformWidgetPanel->SetContent(CustomWidget.ToSharedRef());
				}
				else
				{
					CustomPlatformWidgetPanel->SetContent(SNullWidget::NullWidget);
				}

			}
		}

		FeatureListView->RequestListRefresh();
		QuickInfo->SetDeviceService(DeviceService);
	};

	// wire up models
	Model->OnSelectedDeviceServiceChanged().AddLambda(ModelSelectedDeviceServiceChanged);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
