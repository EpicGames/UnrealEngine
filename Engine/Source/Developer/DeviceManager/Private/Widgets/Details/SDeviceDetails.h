// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Views/SListView.h"

class FDeviceManagerModel;
class SDeviceQuickInfo;

struct FDeviceDetailsFeature;


/**
 * Implements the device details widget.
 */
class SDeviceDetails
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceDetails) { }
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SDeviceDetails();

public:

	/**
	 * Construct the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InModel The view model to use.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FDeviceManagerModel>& InModel);

private:

	/** The list of device features. */
	TArray<TSharedPtr<FDeviceDetailsFeature>> FeatureList;

	/** The device's feature list view. */
	TSharedPtr<SListView<TSharedPtr<FDeviceDetailsFeature>> > FeatureListView;

	/** Pointer the device manager's view model. */
	TSharedPtr<FDeviceManagerModel> Model;

	/** The quick information widget. */
	TSharedPtr<SDeviceQuickInfo> QuickInfo;

	/** Panel on which custom widget will be placed */
	TSharedPtr<SBox> CustomPlatformWidgetPanel;
};
