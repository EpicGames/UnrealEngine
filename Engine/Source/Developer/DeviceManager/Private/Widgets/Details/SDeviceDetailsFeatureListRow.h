// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "Models/DeviceDetailsFeature.h"
#include "SlateOptMacros.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SDeviceDetailsFeatureListRow"

/**
 * Implements a row widget for the device feature list.
 */
class SDeviceDetailsFeatureListRow
	: public SMultiColumnTableRow<TSharedPtr<FDeviceDetailsFeature>>
{
public:

	SLATE_BEGIN_ARGS(SDeviceDetailsFeatureListRow) { }
//		SLATE_ARGUMENT(TSharedPtr<ISlateStyle>, Style)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InOwnerTableView The table view that owns this row.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FDeviceDetailsFeature>& InFeature)
	{
//		check(InArgs._Style.IsValid());

		Feature = InFeature;
//		Style = InArgs._Style;

		SMultiColumnTableRow<TSharedPtr<FDeviceDetailsFeature>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	//~ SMultiColumnTableRow interface

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == "Available")
		{
			const FCoreTexts& CoreTexts = FCoreTexts::Get();
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SDeviceDetailsFeatureListRow::HandleTextColorAndOpacity)
						.Text(Feature->Available ? CoreTexts.Yes : CoreTexts.No)
				];
		}
		else if (ColumnName == "Feature")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SDeviceDetailsFeatureListRow::HandleTextColorAndOpacity)
						.Text(FText::FromString(Feature->FeatureName))
				];
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	/** Callback for getting the text color. */
	FSlateColor HandleTextColorAndOpacity() const
	{
		if (Feature->Available)
		{
			return FSlateColor::UseForeground();
		}

		return FSlateColor::UseSubduedForeground();
	}

private:

	/** Pointer to the device feature to be shown in this row. */
	TSharedPtr<FDeviceDetailsFeature> Feature;

	/** The widget's visual style. */
//	TSharedPtr<ISlateStyle> Style;
};


#undef LOCTEXT_NAMESPACE
