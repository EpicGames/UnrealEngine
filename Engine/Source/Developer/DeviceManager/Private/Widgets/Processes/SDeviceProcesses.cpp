// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Processes/SDeviceProcesses.h"

#include "Styling/AppStyle.h"
#include "Interfaces/ITargetDevice.h"
#include "Internationalization/Text.h"
#include "Misc/MessageDialog.h"
#include "SlateOptMacros.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

#include "Models/DeviceManagerModel.h"
#include "Widgets/Processes/SDeviceProcessesProcessTreeNode.h"
#include "Widgets/Processes/SDeviceProcessesProcessListRow.h"


#define LOCTEXT_NAMESPACE "SDeviceProcesses"


/* SMessagingEndpoints structors
*****************************************************************************/

SDeviceProcesses::~SDeviceProcesses()
{
	if (Model.IsValid())
	{
		Model->OnSelectedDeviceServiceChanged().RemoveAll(this);
	}
}


/* SDeviceDetails interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDeviceProcesses::Construct(const FArguments& InArgs, const TSharedRef<FDeviceManagerModel>& InModel)
{
	Model = InModel;
	ShowProcessTree = true;

	// callback for getting the text of the message overlay.
	auto MessageOverlayText = [this]() -> FText {
		ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

		if (DeviceService.IsValid())
		{
			ITargetDevicePtr Device = DeviceService->GetDevice();

			if (Device.IsValid() && Device->IsConnected())
			{
				if (Device->SupportsFeature(ETargetDeviceFeatures::ProcessSnapshot))
				{
					return FText::GetEmpty();
				}

				return LOCTEXT("ProcessSnapshotsUnsupportedOverlayText", "The selected device does not support process snapshots");
			}

			return LOCTEXT("DeviceUnavailableOverlayText", "The selected device is currently unavailable");
		}

		return LOCTEXT("SelectDeviceOverlayText", "Please select a device from the Device Browser");
	};

	// callback for getting the visibility of the message overlay.
	auto MessageOverlayVisibility = [this]() -> EVisibility {
		ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

		if (DeviceService.IsValid())
		{
			ITargetDevicePtr Device = DeviceService->GetDevice();

			if (Device.IsValid() && Device->IsConnected() && Device->SupportsFeature(ETargetDeviceFeatures::ProcessSnapshot))
			{
				return EVisibility::Hidden;
			}
		}

		return EVisibility::Visible;
	};

	// construct children
	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
			[
				SNew(SVerticalBox)
					.IsEnabled(this, &SDeviceProcesses::HandleProcessesBoxIsEnabled)

				+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(0.0f, 4.0f, 0.0f, 0.0f)
					[
						// process list
						SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							.Padding(0.0f)
							[
								SAssignNew(ProcessTreeView, STreeView<TSharedPtr<FDeviceProcessesProcessTreeNode>>)
									.OnGenerateRow_Lambda(
										[](TSharedPtr<FDeviceProcessesProcessTreeNode> Item, const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow> {
											return SNew(SDeviceProcessesProcessListRow, OwnerTable)
												.Node(Item);
										}
									)
									.OnGetChildren_Lambda(
										[](TSharedPtr<FDeviceProcessesProcessTreeNode> Item, TArray<TSharedPtr<FDeviceProcessesProcessTreeNode>>& OutChildren) {
											if (Item.IsValid())
											{
												OutChildren = Item->GetChildren();
											}
										}
									)
									.SelectionMode(ESelectionMode::Multi)
									.TreeItemsSource(&ProcessList)
									.HeaderRow
									(
										SNew(SHeaderRow)

										+ SHeaderRow::Column("Name")
											.DefaultLabel(LOCTEXT("ProcessListNameColumnHeader", "Process Name"))
											.FillWidth(0.275f)

										+ SHeaderRow::Column("PID")
											.DefaultLabel(LOCTEXT("ProcessListPidColumnHeader", "PID"))
											.FillWidth(0.15f)

										+ SHeaderRow::Column("User")
											.DefaultLabel(LOCTEXT("ProcessListUserColumnHeader", "User"))
											.FillWidth(0.275f)

										+ SHeaderRow::Column("Threads")
											.DefaultLabel(LOCTEXT("ProcessListThreadsColumnHeader", "Threads"))
											.FillWidth(0.15f)

										+ SHeaderRow::Column("Parent")
											.DefaultLabel(LOCTEXT("ProcessListParentColumnHeader", "Parent PID"))
											.FillWidth(0.15f)
									)
							]
					]

				+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 4.0f, 0.0f, 0.0f)
					[
						SNew(SBorder)
							.Padding(FMargin(8.0f, 6.0f, 8.0f, 4.0f))
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									[
										SNew(SCheckBox)
											.IsChecked_Lambda(
												[this]() -> ECheckBoxState {
													return ShowProcessTree ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
												}
											)
											.OnCheckStateChanged_Lambda(
												[this](ECheckBoxState NewState) {
													ShowProcessTree = (NewState == ECheckBoxState::Checked);
													ReloadProcessList(false);
												}
											)
											.Padding(FMargin(4.0f, 0.0f))
											.ToolTipText(LOCTEXT("ProcessTreeCheckBoxToolTip", "Check this box to display the list of processes as a tree instead of a flat list"))
											[
												SNew(STextBlock)
													.Text(LOCTEXT("ProcessTreeCheckBoxText", "Show process tree"))
											]
									]

								+ SHorizontalBox::Slot()
									.HAlign(HAlign_Right)
									[
										SNew(SButton)
											.IsEnabled_Lambda(
												[this]() -> bool {
													return (ProcessTreeView->GetNumItemsSelected() > 0);
												}
											)
											.OnClicked(this, &SDeviceProcesses::HandleTerminateProcessButtonClicked)
											.Text(LOCTEXT("TerminateProcessButtonText", "Terminate Process"))
									]
							]
					]
			]

		+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NotificationList.ItemBackground"))
					.Padding(8.0f)
					.Visibility_Lambda(MessageOverlayVisibility)
					[
						SNew(STextBlock)
							.Text_Lambda(MessageOverlayText)
					]
			]
	];

	// callback for handling device service selection changes.
	auto ModelSelectedDeviceServiceChanged = [this]() {
		ReloadProcessList(true);
	};

	// wire up models
	Model->OnSelectedDeviceServiceChanged().AddLambda([this]() {  });

	// initialize
	ReloadProcessList(true);
	RegisterActiveTimer(2.5f, FWidgetActiveTimerDelegate::CreateSP(this, &SDeviceProcesses::UpdateProcessList));
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SDeviceDetails implementation
 *****************************************************************************/

void SDeviceProcesses::ReloadProcessList(bool FullyReload)
{
	bool Reloaded = false;
	// reload running processes
	if (FullyReload)
	{
		RunningProcesses.Reset();

		ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

		if (DeviceService.IsValid())
		{
			ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

			if (TargetDevice.IsValid())
			{
				TWeakPtr<SDeviceProcesses> WeakSelf = SharedThis(this);

				Reloaded = TargetDevice->GetProcessSnapshotAsync([this, WeakSelf](const TArray<FTargetDeviceProcessInfo>& InProcessInfos)
					{ 
						check(IsInGameThread());
						TSharedPtr<SDeviceProcesses> SharedSelf = WeakSelf.Pin();
						if (!SharedSelf.IsValid())
						{
							//this pointer isn't valid any more
							return;
						}
						RunningProcesses = InProcessInfos;
						UpdateProcessTree(); 
					});
			}
		}
	}

	if (!Reloaded)
	{
		UpdateProcessTree();
	}
}

void SDeviceProcesses::UpdateProcessTree()
{
	// update process tree
	SDeviceProcesses::ProcessMapType NewProcessMap;

	for (int32 ProcessIndex = 0; ProcessIndex < RunningProcesses.Num(); ++ProcessIndex)
	{
		const FTargetDeviceProcessInfo& ProcessInfo = RunningProcesses[ProcessIndex];
		TSharedPtr<FDeviceProcessesProcessTreeNode> Node = ProcessMap.FindRef(ProcessInfo.Id);

		if (Node.IsValid())
		{
			Node->ClearChildren();
			Node->SetParent(NULL);

			NewProcessMap.Add(ProcessInfo.Id, Node);
		}
		else
		{
			NewProcessMap.Add(ProcessInfo.Id, MakeShareable(new FDeviceProcessesProcessTreeNode(ProcessInfo)));
		}		
	}

	ProcessMap = NewProcessMap;

	// build process tree
	if (ShowProcessTree)
	{
		for (SDeviceProcesses::ProcessMapType::TConstIterator It(ProcessMap); It; ++It)
		{
			TSharedPtr<FDeviceProcessesProcessTreeNode> Node = It.Value();
			TSharedPtr<FDeviceProcessesProcessTreeNode> Parent = ProcessMap.FindRef((int64)Node->GetProcessInfo().ParentId);

			if (Parent.IsValid())
			{
				Node->SetParent(Parent);
				Parent->AddChild(Node);
			}
		}
	}

	// filter process list
	ProcessList.Reset();

	for (SDeviceProcesses::ProcessMapType::TConstIterator It(ProcessMap); It; ++It)
	{
		TSharedPtr<FDeviceProcessesProcessTreeNode> Node = It.Value();

		if (!Node->GetParent().IsValid())
		{
			ProcessList.Add(Node);
		}
	}

	// refresh list view
	ProcessTreeView->RequestTreeRefresh();

	LastProcessListRefreshTime = FDateTime::UtcNow();
}


EActiveTimerReturnType SDeviceProcesses::UpdateProcessList(double InCurrentTime, float InDeltaTime)
{
	ReloadProcessList(true);

	return EActiveTimerReturnType::Continue;
}


/* SDeviceDetails callbacks
 *****************************************************************************/

bool SDeviceProcesses::HandleProcessesBoxIsEnabled() const
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		ITargetDevicePtr Device = DeviceService->GetDevice();

		return (Device.IsValid() && Device->IsConnected() && Device->SupportsFeature(ETargetDeviceFeatures::ProcessSnapshot));
	}

	return false;
}


FReply SDeviceProcesses::HandleTerminateProcessButtonClicked()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		if (FMessageDialog::Open(EAppMsgType::OkCancel, LOCTEXT("TerminateProcessWarning", "Warning: If you terminate a process that is associated with a game or an application, you willl lose any unsaved data. If you end a system process, it might result in an unstable system.")) == EAppReturnType::Ok)
		{
			TArray<TSharedPtr<FDeviceProcessesProcessTreeNode>> FailedProcesses;
			TArray<TSharedPtr<FDeviceProcessesProcessTreeNode>> SelectedProcesses = ProcessTreeView->GetSelectedItems();

			for (int32 ProcessIndex = 0; ProcessIndex < SelectedProcesses.Num(); ++ProcessIndex)
			{
				ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

				if (TargetDevice.IsValid())
				{
					const TSharedPtr<FDeviceProcessesProcessTreeNode>& Process = SelectedProcesses[ProcessIndex];

					if (!TargetDevice->TerminateProcess(Process->GetProcessInfo().Id))
					{
						FailedProcesses.Add(Process);
					}
				}
			}

			if (FailedProcesses.Num() > 0)
			{
				FString ProcessInfo;
				for (int32 FailedProcessIndex = 0; FailedProcessIndex < FailedProcesses.Num(); ++FailedProcessIndex)
				{
					const FTargetDeviceProcessInfo& FailedProcessInfo = FailedProcesses[FailedProcessIndex]->GetProcessInfo();
					ProcessInfo += FString::Printf(TEXT("%s (PID: %" INT64_FMT ")\n"), *FailedProcessInfo.Name, FailedProcessInfo.Id);
				}

				const FText ErrorMessage = FText::Format(LOCTEXT("FailedToTerminateProcessesMessage", "The following processes could not be terminated.\nYou may not have the required permissions:\n\n{0}"), FText::FromString(ProcessInfo));
				FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
			}
		}
	}

	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
