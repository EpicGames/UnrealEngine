// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Misc/DateTime.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class FDeviceManagerModel;
class FDeviceProcessesProcessTreeNode;

struct FTargetDeviceProcessInfo;


/**
 * Implements the device details widget.
 */
class SDeviceProcesses
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceProcesses) { }
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SDeviceProcesses();

public:

	/**
	 * Construct the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InModel The view model to use.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FDeviceManagerModel>& InModel);

	typedef TMap<int64, TSharedPtr<FDeviceProcessesProcessTreeNode>> ProcessMapType;

protected:

	/**
	 * Reload the list of processes.
	 *
	 * @param FullyReload true reload all entries, false to refresh the existing entries.
	 */
	void ReloadProcessList(bool FullyReload);

	void UpdateProcessTree();

	/** Periodically refreshes the process list. */
	EActiveTimerReturnType UpdateProcessList(double InCurrentTime, float InDeltaTime);

private:

	/** Callback for getting the enabled state of the processes panel. */
	bool HandleProcessesBoxIsEnabled() const;

	/** Callback for clicking the 'Terminate Process' button. */
	FReply HandleTerminateProcessButtonClicked();

private:
	
	/** Holds the time at which the process list was last refreshed. */
	FDateTime LastProcessListRefreshTime;

	/** Holds a pointer the device manager's view model. */
	TSharedPtr<FDeviceManagerModel> Model;

	/** Holds the filtered list of processes running on the device. */
	TArray<TSharedPtr<FDeviceProcessesProcessTreeNode>> ProcessList;

	/** Holds the collection of process nodes. */
	ProcessMapType ProcessMap;

	/** Holds the application tree view. */
	TSharedPtr<STreeView<TSharedPtr<FDeviceProcessesProcessTreeNode>>> ProcessTreeView;

	/** Holds the collection of processes running on the device. */
	TArray<FTargetDeviceProcessInfo> RunningProcesses;

	/** Holds a flag indicating whether the process list should be shown as a tree instead of a list. */
	bool ShowProcessTree;
};
