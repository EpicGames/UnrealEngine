// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Layout/Margin.h"
#include "SlateOptMacros.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#include "Widgets/Processes/SDeviceProcessesProcessTreeNode.h"


#define LOCTEXT_NAMESPACE "SDeviceProcessesProcessListRow"


/**
 * Implements a row widget for the process list view.
 */
class SDeviceProcessesProcessListRow
	: public SMultiColumnTableRow<TSharedPtr<FDeviceProcessesProcessTreeNode>>
{
public:

	SLATE_BEGIN_ARGS(SDeviceProcessesProcessListRow) { }
		SLATE_ARGUMENT(TSharedPtr<FDeviceProcessesProcessTreeNode>, Node)
	SLATE_END_ARGS()

public:

	/**
	 * Construct the widget.
	 *
	 * @param InArgs The arguments.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Node = InArgs._Node;

		SMultiColumnTableRow<TSharedPtr<FDeviceProcessesProcessTreeNode>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	//~ SMultiColumnTableRow interface

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == TEXT("Name"))
		{
			TSharedRef<SWidget> ColumnContent = SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(FText::FromString(Node->GetProcessInfo().Name))
				];

			if (OwnerTablePtr.Pin()->GetTableViewMode() == ETableViewMode::Tree)
			{
				return 
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Top)
						[
							SNew(SExpanderArrow, SharedThis(this))
						]
					
					+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							ColumnContent
						];
			}
			else
			{
				return ColumnContent;
			}
		}
		else if (ColumnName == TEXT("Parent"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(FText::AsNumber(Node->GetProcessInfo().ParentId))
				];
		}
		else if (ColumnName == TEXT("PID"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(FText::AsNumber(Node->GetProcessInfo().Id))
				];
		}
		else if (ColumnName == TEXT("Threads"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(FText::AsNumber(Node->GetProcessInfo().Threads.Num()))
				];
		}
		else if (ColumnName == TEXT("User"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(FText::FromString(Node->GetProcessInfo().UserName))
				];
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	/** The name of the process. */
	TSharedPtr<FDeviceProcessesProcessTreeNode> Node;
};


#undef LOCTEXT_NAMESPACE
