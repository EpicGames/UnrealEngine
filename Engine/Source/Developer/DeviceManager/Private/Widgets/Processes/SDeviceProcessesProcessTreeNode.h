// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Interfaces/ITargetDevice.h"
#include "Templates/SharedPointer.h"

class FDeviceProcessesProcessTreeNode;


/**
 * Implements a node for the process tree view.
 */
class FDeviceProcessesProcessTreeNode
{
public:

	/**
	 * Create and initializes a new instance.
	 *
	 * @param InProcessInfo The node's process information.
	 */
	FDeviceProcessesProcessTreeNode(const FTargetDeviceProcessInfo& InProcessInfo)
		: ProcessInfo(InProcessInfo)
	{ }

public:

	/**
	 * Add a child process node to this node.
	 *
	 * @param The child node to add.
	 * @see ClearChildren, GetChildren
	 */
	void AddChild(const TSharedPtr<FDeviceProcessesProcessTreeNode>& Child)
	{
		Children.Add(Child);
	}

	/**
	 * Clear the collection of child nodes.
	 *
	 * @see AddChild, GetChildren
	 */
	void ClearChildren()
	{
		Children.Reset();
	}

	/**
	 * Get the child nodes.
	 *
	 * @return Child nodes.
	 * @see GetParent
	 */
	const TArray<TSharedPtr<FDeviceProcessesProcessTreeNode>>& GetChildren()
	{
		return Children;
	}

	/**
	 * Get the parent node.
	 *
	 * @return Parent node.
	 * @see GetChildren, SetParent
	 */
	const TSharedPtr<FDeviceProcessesProcessTreeNode>& GetParent()
	{
		return Parent;
	}

	/**
	 * Get the node's process information.
	 *
	 * @return The process information.
	 */
	const FTargetDeviceProcessInfo& GetProcessInfo() const
	{
		return ProcessInfo;
	}

	/**
	 * Set the parent node.
	 *
	 * @param Node The parent node to set.
	 * @see GetParent
	 */
	void SetParent(const TSharedPtr<FDeviceProcessesProcessTreeNode>& Node)
	{
		Parent = Node;
	}

private:

	/** The child process nodes. */
	TArray<TSharedPtr<FDeviceProcessesProcessTreeNode>> Children;

	/** Pointer to the parent node. */
	TSharedPtr<FDeviceProcessesProcessTreeNode> Parent;

	/** The process information. */
	FTargetDeviceProcessInfo ProcessInfo;
};
