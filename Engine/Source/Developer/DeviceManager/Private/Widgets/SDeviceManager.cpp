// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDeviceManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/MessageDialog.h"
#include "Widgets/SBoxPanel.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "Interfaces/ITargetPlatform.h"
#include "Models/DeviceDetailsCommands.h"
#include "Widgets/Apps/SDeviceApps.h"
#include "Widgets/Browser/SDeviceBrowser.h"
#include "Widgets/Details/SDeviceDetails.h"
#include "Widgets/Processes/SDeviceProcesses.h"
#include "Widgets/Toolbar/SDeviceToolbar.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"


#define LOCTEXT_NAMESPACE "SDeviceManager"


/* Local constants
 *****************************************************************************/

static const FName DeviceAppsTabId("DeviceApps");
static const FName DeviceBrowserTabId("DeviceBrowser");
static const FName DeviceDetailsTabId("DeviceDetails");
static const FName DeviceProcessesTabId("DeviceProcesses");
static const FName DeviceToolbarTabId("DeviceToolbar");


/* SDeviceManager constructors
 *****************************************************************************/

SDeviceManager::SDeviceManager()
	: Model(MakeShareable(new FDeviceManagerModel()))
{ }


/* SDeviceManager interface
 *****************************************************************************/

void SDeviceManager::Construct(const FArguments& InArgs, const TSharedRef<ITargetDeviceServiceManager>& InDeviceServiceManager, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	DeviceServiceManager = InDeviceServiceManager;

	// create and bind UI commands
	FDeviceDetailsCommands::Register();
	UICommandList = MakeShareable(new FUICommandList);
	BindCommands();

	// create & initialize tab manager
	TabManager = FGlobalTabmanager::Get()->NewTabManager(ConstructUnderMajorTab);
	TSharedRef<FWorkspaceItem> AppMenuGroup = TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("DeviceManagerMenuGroupName", "Device Manager"));
	TabManager->SetAllowWindowMenuBar(true);

	TabManager->RegisterTabSpawner(DeviceBrowserTabId, FOnSpawnTab::CreateRaw(this, &SDeviceManager::HandleTabManagerSpawnTab, DeviceBrowserTabId))
		.SetDisplayName(LOCTEXT("DeviceBrowserTabTitle", "Device Browser"))
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Settings"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(DeviceDetailsTabId, FOnSpawnTab::CreateRaw(this, &SDeviceManager::HandleTabManagerSpawnTab, DeviceDetailsTabId))
		.SetDisplayName(LOCTEXT("DeviceDetailsTabTitle", "Device Details"))
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Settings"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(DeviceAppsTabId, FOnSpawnTab::CreateRaw(this, &SDeviceManager::HandleTabManagerSpawnTab, DeviceAppsTabId))
		.SetDisplayName(LOCTEXT("DeviceAppsTabTitle", "Deployed Apps"))
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Settings"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(DeviceProcessesTabId, FOnSpawnTab::CreateRaw(this, &SDeviceManager::HandleTabManagerSpawnTab, DeviceProcessesTabId))
		.SetDisplayName(LOCTEXT("DeviceProcessesTabTitle", "Running Processes"))
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Settings"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(DeviceToolbarTabId, FOnSpawnTab::CreateRaw(this, &SDeviceManager::HandleTabManagerSpawnTab, DeviceToolbarTabId))
		.SetDisplayName(LOCTEXT("DeviceToolbarTabTitle", "Toolbar"))
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Settings"))
		.SetGroup(AppMenuGroup);

	// create tab layout
	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("DeviceManagerLayout_v1.1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
						->AddTab(DeviceToolbarTabId, ETabState::OpenedTab)
						->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
						->AddTab(DeviceBrowserTabId, ETabState::OpenedTab)
						->SetHideTabWell(true)
						->SetSizeCoefficient(0.5f)
				)
				->Split
				(
					FTabManager::NewSplitter()
						->SetOrientation(Orient_Horizontal)
						->SetSizeCoefficient(0.5f)
						->Split
						(
							FTabManager::NewStack()
								->AddTab(DeviceAppsTabId, ETabState::ClosedTab)
								->AddTab(DeviceProcessesTabId, ETabState::OpenedTab)
								->SetSizeCoefficient(0.75f)
						)
						->Split
						(
							FTabManager::NewStack()
								->AddTab(DeviceDetailsTabId, ETabState::OpenedTab)
								->SetSizeCoefficient(0.25f)
						)
				)
		);

	// create & initialize main menu
	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());

	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateStatic(&SDeviceManager::FillWindowMenu, TabManager),
		"Window"
	);

	TSharedRef<SWidget> MenuWidget = MenuBarBuilder.MakeWidget();

	// construct children
	ChildSlot
	[
		TabManager->RestoreFrom(Layout, ConstructUnderWindow).ToSharedRef()
	];

	// Tell tab-manager about the multi-box for platforms with a global menu bar
	TabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuWidget);
}


/* SDeviceManager implementation
 *****************************************************************************/

void SDeviceManager::BindCommands()
{
	const FDeviceDetailsCommands& Commands = FDeviceDetailsCommands::Get();

	// ownership commands
	UICommandList->MapAction(
		Commands.Claim,
		FExecuteAction::CreateSP(this, &SDeviceManager::HandleClaimActionExecute),
		FCanExecuteAction::CreateSP(this, &SDeviceManager::HandleClaimActionCanExecute));

	UICommandList->MapAction(
		Commands.Release,
		FExecuteAction::CreateSP(this, &SDeviceManager::HandleReleaseActionExecute),
		FCanExecuteAction::CreateSP(this, &SDeviceManager::HandleReleaseActionCanExecute));

	UICommandList->MapAction(
		Commands.Remove,
		FExecuteAction::CreateSP(this, &SDeviceManager::HandleRemoveActionExecute),
		FCanExecuteAction::CreateSP(this, &SDeviceManager::HandleRemoveActionCanExecute));

	UICommandList->MapAction(
		Commands.Share,
		FExecuteAction::CreateSP(this, &SDeviceManager::HandleShareActionExecute),
		FCanExecuteAction::CreateSP(this, &SDeviceManager::HandleShareActionCanExecute),
		FIsActionChecked::CreateSP(this, &SDeviceManager::HandleShareActionIsChecked));

	// connectivity commands
	UICommandList->MapAction(
		Commands.Connect, 
		FExecuteAction::CreateSP(this, &SDeviceManager::HandleConnectActionExecute),
		FCanExecuteAction::CreateSP(this, &SDeviceManager::HandleConnectActionCanExecute));

	UICommandList->MapAction(
		Commands.Disconnect, 
		FExecuteAction::CreateSP(this, &SDeviceManager::HandleDisconnectActionExecute),
		FCanExecuteAction::CreateSP(this, &SDeviceManager::HandleDisconnectActionCanExecute));

	// remote control commands
	UICommandList->MapAction(
		Commands.PowerOff,
		FExecuteAction::CreateSP(this, &SDeviceManager::HandlePowerOffActionExecute, false),
		FCanExecuteAction::CreateSP(this, &SDeviceManager::HandlePowerOffActionCanExecute));

	UICommandList->MapAction(
		Commands.PowerOffForce,
		FExecuteAction::CreateSP(this, &SDeviceManager::HandlePowerOffActionExecute, true),
		FCanExecuteAction::CreateSP(this, &SDeviceManager::HandlePowerOffActionCanExecute));

	UICommandList->MapAction(
		Commands.PowerOn,
		FExecuteAction::CreateSP(this, &SDeviceManager::HandlePowerOnActionExecute),
		FCanExecuteAction::CreateSP(this, &SDeviceManager::HandlePowerOnActionCanExecute));

	UICommandList->MapAction(
		Commands.Reboot,
		FExecuteAction::CreateSP(this, &SDeviceManager::HandleRebootActionExecute),
		FCanExecuteAction::CreateSP(this, &SDeviceManager::HandleRebootActionCanExecute));
}


void SDeviceManager::FillWindowMenu(FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager)
{
	if (!TabManager.IsValid())
	{
		return;
	}

#if !WITH_EDITOR
		FGlobalTabmanager::Get()->PopulateTabSpawnerMenu(MenuBuilder, WorkspaceMenu::GetMenuStructure().GetStructureRoot());
#endif //!WITH_EDITOR

	TabManager->PopulateLocalTabSpawnerMenu(MenuBuilder);
}


bool SDeviceManager::ValidateDeviceAction(const ITargetDeviceRef& Device) const
{
	// @todo gmp: this needs to be improved, i.e. TargetPlatformManager::GetLocalDevice
	if (Device->GetName() != FPlatformProcess::ComputerName())
	{
		return true;
	}

	int32 DialogResult = FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("LocalHostDialogPrompt", "WARNING: This device represents your local computer.\n\nAre you sure you want to proceed?"));

	return (DialogResult == EAppReturnType::Yes);
}


/* SDeviceManager callbacks
 *****************************************************************************/

bool SDeviceManager::HandleClaimActionCanExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	return (DeviceService.IsValid() && !DeviceService->IsRunning() && DeviceService->GetClaimUser().IsEmpty());
}


void SDeviceManager::HandleClaimActionExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		DeviceService->Start();
	}
}


bool SDeviceManager::HandleConnectActionCanExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

		if (TargetDevice.IsValid() && TargetDevice->GetPlatformSettings().SupportsFeature(ETargetPlatformFeatures::SdkConnectDisconnect))
		{
			return !TargetDevice->IsConnected();
		}
	}

	return false;
}


void SDeviceManager::HandleConnectActionExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

		if (TargetDevice.IsValid())
		{
			if (!TargetDevice->Connect())
			{
				FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("DeviceConnectFailedMessage", "Unable to connect to the device. Please make sure that it is powered on!"));
			}
		}
	}
}


bool SDeviceManager::HandleDisconnectActionCanExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

		if (TargetDevice.IsValid() && TargetDevice->GetPlatformSettings().SupportsFeature(ETargetPlatformFeatures::SdkConnectDisconnect))
		{
			return TargetDevice->IsConnected();
		}
	}

	return false;
}


void SDeviceManager::HandleDisconnectActionExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

		if (TargetDevice.IsValid())
		{
			TargetDevice->Disconnect();
		}
	}
}


bool SDeviceManager::HandlePowerOffActionCanExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

		if (TargetDevice.IsValid())
		{
			return (TargetDevice->IsConnected() && TargetDevice->SupportsFeature(ETargetDeviceFeatures::PowerOff));
		}
	}

	return false;
}


void SDeviceManager::HandlePowerOffActionExecute(bool Force)
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

		if (TargetDevice.IsValid() && ValidateDeviceAction(TargetDevice.ToSharedRef()))
		{
			TargetDevice->PowerOff(Force);
		}
	}
}


bool SDeviceManager::HandlePowerOnActionCanExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

		if (TargetDevice.IsValid())
		{
			return (TargetDevice->SupportsFeature(ETargetDeviceFeatures::PowerOn));
		}
	}

	return false;
}


void SDeviceManager::HandlePowerOnActionExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

		if (TargetDevice.IsValid())
		{
			TargetDevice->PowerOn();
		}
	}
}


bool SDeviceManager::HandleRebootActionCanExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

		if (TargetDevice.IsValid())
		{
			return (TargetDevice->IsConnected() && TargetDevice->SupportsFeature(ETargetDeviceFeatures::Reboot));
		}
	}

	return false;
}


void SDeviceManager::HandleRebootActionExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

		if (TargetDevice.IsValid() && ValidateDeviceAction(TargetDevice.ToSharedRef()))
		{
			TargetDevice->Reboot(true);
		}
	}
}


bool SDeviceManager::HandleReleaseActionCanExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	return (DeviceService.IsValid() && DeviceService->IsRunning());
}


void SDeviceManager::HandleReleaseActionExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		DeviceService->Stop();
	}
}


bool SDeviceManager::HandleRemoveActionCanExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	// @todo gmp: at some point support removal of available devices through their SDK (i.e. remove from PS4 neighborhood)
	return (DeviceService.IsValid() && !DeviceService->GetDevice().IsValid());
}


void SDeviceManager::HandleRemoveActionExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid())
	{
		DeviceServiceManager->RemoveStartupService(DeviceService->GetDeviceName());
	}
}


bool SDeviceManager::HandleShareActionIsChecked()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	return (DeviceService.IsValid() && DeviceService->IsShared());
}


void SDeviceManager::HandleShareActionExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	if (DeviceService.IsValid() && DeviceService->IsRunning())
	{
		DeviceService->SetShared(!DeviceService->IsShared());
	}
}


bool SDeviceManager::HandleShareActionCanExecute()
{
	ITargetDeviceServicePtr DeviceService = Model->GetSelectedDeviceService();

	return (DeviceService.IsValid() && DeviceService->IsRunning());
}


TSharedRef<SDockTab> SDeviceManager::HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier)
{
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;
	bool AutoSizeTab = false;

	if (TabIdentifier == DeviceAppsTabId)
	{
		TabWidget = SNew(SDeviceApps, Model);
	}
	else if (TabIdentifier == DeviceBrowserTabId)
	{
		TabWidget = SNew(SDeviceBrowser, Model, DeviceServiceManager.ToSharedRef(), UICommandList);
	}
	else if (TabIdentifier == DeviceDetailsTabId)
	{
		TabWidget = SNew(SDeviceDetails, Model);
	}
	else if (TabIdentifier == DeviceProcessesTabId)
	{
		TabWidget = SNew(SDeviceProcesses, Model);
	}
	else if (TabIdentifier == DeviceToolbarTabId)
	{
		TabWidget = SNew(SDeviceToolbar, Model, UICommandList);
		AutoSizeTab = true;
	}

	return SNew(SDockTab)
		.ShouldAutosize(AutoSizeTab)
		.TabRole(ETabRole::PanelTab)
		[
			TabWidget.ToSharedRef()
		];
}


#undef LOCTEXT_NAMESPACE
