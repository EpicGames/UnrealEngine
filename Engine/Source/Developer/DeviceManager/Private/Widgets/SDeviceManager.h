// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "ITargetDeviceServiceManager.h"
#include "Models/DeviceManagerModel.h"

class FMenuBuilder;
class FSpawnTabArgs;
class FTabManager;
class FUICommandList;
class SWindow;


/**
 * Implements the device manager front-end widget.
 */
class SDeviceManager
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceManager) { }
	SLATE_END_ARGS()

public:

	/** Default constructor. */
	SDeviceManager();

public:

	/**
	 * Construct the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InDeviceServiceManager The target device manager to use.
	 * @param ConstructUnderMajorTab The major tab which will contain the session front-end.
	 * @param ConstructUnderWindow The window in which this widget is being constructed.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<ITargetDeviceServiceManager>& InDeviceServiceManager, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

protected:

	/**
	 * Bind the device commands on our toolbar.
	 */
	void BindCommands();

	/**
	 * Fill the Window menu with menu items.
	 *
	 * @param MenuBuilder The multi-box builder that should be filled with content for this pull-down menu.
	 * @param TabManager A Tab Manager from which to populate tab spawner menu items.
	 */
	static void FillWindowMenu(FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager);

	/**
	 * Validate actions on the specified device.
	 *
	 * @param Device The device to perform an action on.
	 * @return true if actions on the device are permitted, false otherwise.
	 */
	bool ValidateDeviceAction(const ITargetDeviceRef& Device) const;

private:

	/** Callback for determining whether the 'Claim' action can execute. */
	bool HandleClaimActionCanExecute();

	/** Callback for executing the 'Claim' action. */
	void HandleClaimActionExecute();

	/** Callback for determining whether the "Connect" action can execute. */
	bool HandleConnectActionCanExecute();

	/** Callback for executing the "Connect" action. */
	void HandleConnectActionExecute();

	/** Callback for determining whether the "Disconnect" action can execute. */
	bool HandleDisconnectActionCanExecute();

	/** Callback for executing the "Disconnect" action. */
	void HandleDisconnectActionExecute();

	/** Callback for determining whether the 'Power Off' action can execute. */
	bool HandlePowerOffActionCanExecute();

	/** Callback for executing the 'Power Off' action. */
	void HandlePowerOffActionExecute(bool Force);

	/** Callback for determining whether the 'Power On' action can execute. */
	bool HandlePowerOnActionCanExecute();

	/** Callback for executing the 'Power On' action. */
	void HandlePowerOnActionExecute();

	/** Callback for determining whether the 'Reboot' action can execute. */
	bool HandleRebootActionCanExecute();

	/** Callback for executing the 'Reboot' action. */
	void HandleRebootActionExecute();

	/** Callback for determining whether the 'Release' action can execute. */
	bool HandleReleaseActionCanExecute();

	/** Callback for executing the 'Release' action. */
	void HandleReleaseActionExecute();

	/** Callback for determining whether the 'Remove' action can execute. */
	bool HandleRemoveActionCanExecute();

	/** Callback for executing the 'Remove' action. */
	void HandleRemoveActionExecute();

	/** Callback for determining whether the 'Share' action can execute. */
	bool HandleShareActionCanExecute();

	/** Callback for executing the 'Share' action. */
	void HandleShareActionExecute();

	/** Callback for determining the checked state of the 'Share' action. */
	bool HandleShareActionIsChecked();

	/** Callback for spawning tabs. */
	TSharedRef<SDockTab> HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier);

private:

	/** Holds the target device service manager. */
	TSharedPtr<ITargetDeviceServiceManager> DeviceServiceManager;

	/** Holds the device manager's view model. */
	TSharedRef<FDeviceManagerModel> Model;

	/** Holds the tab manager that manages the front-end's tabs. */
	TSharedPtr<FTabManager> TabManager;

	/** The command list for controlling the device. */
	TSharedPtr<FUICommandList> UICommandList;
};
