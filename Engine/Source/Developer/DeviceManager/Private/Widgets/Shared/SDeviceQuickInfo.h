// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/AppStyle.h"
#include "Fonts/SlateFontInfo.h"
#include "ITargetDeviceService.h"
#include "Misc/Paths.h"
#include "PlatformInfo.h"
#include "Styling/StyleDefaults.h"
#include "Styling/CoreStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SDeviceQuickInfo"


/**
 * Implements a tool tip for widget the device browser.
 */
class SDeviceQuickInfo
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceQuickInfo) { }
		
		/** The device service to show the information for. */
		SLATE_ATTRIBUTE(ITargetDeviceServicePtr, InitialDeviceService)

	SLATE_END_ARGS()

public:

	/**
	 * Construct the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InDeviceServiceManager The target device service manager to use.
	 * @param InDeviceManagerState The optional device manager view state.
	 */
	void Construct(const FArguments& InArgs)
	{
		DeviceService = InArgs._InitialDeviceService.Get();

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
						.HeightOverride(96.0f)
						.WidthOverride(96.0f)
						[
							SNew(SImage)
								.Image(this, &SDeviceQuickInfo::HandlePlatformIcon)
						]
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(20.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SGridPanel)
						.FillColumn(0, 1.0f)

					// name
					+ SGridPanel::Slot(0, 0)
						[
							SNew(STextBlock)
								.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
								.Text(LOCTEXT("DeviceNameLabel", "Name:"))
						]

					+ SGridPanel::Slot(1, 0)
						.Padding(16.0f, 0.0f, 8.0f, 0.0f)
						[
							SNew(STextBlock)
								.Text(this, &SDeviceQuickInfo::HandleDeviceNameText)
						]

					// platform
					+ SGridPanel::Slot(0, 1)
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
								.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
								.Text(LOCTEXT("DevicePlatformLabel", "Platform:"))
						]

					+ SGridPanel::Slot(1, 1)
						.Padding(16.0f, 4.0f, 8.0f, 0.0f)
						[
							SNew(STextBlock)
								.Text(this, &SDeviceQuickInfo::HandlePlatformNameText)
						]

					// operating system
					+ SGridPanel::Slot(0, 2)
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
								.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
								.Text(LOCTEXT("DeviceMakeModelLabel", "Operating System:"))
						]

					+ SGridPanel::Slot(1, 2)
						.Padding(16.0f, 4.0f, 8.0f, 0.0f)
						[
							SNew(STextBlock)
								.Text(this, &SDeviceQuickInfo::HandleOperatingSystemText)
						]

					// device identifier
					+ SGridPanel::Slot(0, 3)
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
								.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
								.Text(LOCTEXT("DeviceIdLabel", "Device ID:"))
						]

					+ SGridPanel::Slot(1, 3)
						.Padding(16.0f, 4.0f, 8.0f, 0.0f)
						[
							SNew(STextBlock)
								.Text(this, &SDeviceQuickInfo::HandleDeviceIdText)
						]

					// default device
					+ SGridPanel::Slot(0, 4)
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
								.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
								.Text(LOCTEXT("DefaultDeviceLabel", "Default device:"))
						]

					+ SGridPanel::Slot(1, 4)
						.Padding(16.0f, 4.0f, 8.0f, 0.0f)
						[
							SNew(STextBlock)
								.Text(this, &SDeviceQuickInfo::HandleIsDefaultText)
						]

					// status
					+ SGridPanel::Slot(0, 5)
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
								.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
								.Text(LOCTEXT("StatusLabel", "Status:"))
						]

					+ SGridPanel::Slot(1, 5)
						.Padding(16.0f, 4.0f, 8.0f, 0.0f)
						[
							SNew(STextBlock)
								.Text(this, &SDeviceQuickInfo::HandleStatusText)
						]
				]
		];
	}

public:

	/**
	 * Set the device service whose information is being shown.
	 *
	 * @param InDeviceService The device service to show.
	 */
	void SetDeviceService(const TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe>& InDeviceService)
	{
		DeviceService = InDeviceService;
	}

private:

	/** Callback for getting the device's unique identifier. */
	FText HandleDeviceIdText() const
	{
		if (DeviceService.IsValid())
		{
			ITargetDevicePtr Device = DeviceService->GetDevice();
			if (Device.IsValid())
			{
				return FText::FromString(Device->GetId().ToString());
			}
		}

		return LOCTEXT("UnknownValue", "<unknown>");
	}

	/** Callback for getting the name of the shown device. */
	FText HandleDeviceNameText() const
	{
		if (DeviceService.IsValid())
		{
			FString DeviceName = DeviceService->GetDeviceName();

			if (!DeviceName.IsEmpty())
			{
				return FText::FromString(DeviceName);
			}
		}

		return LOCTEXT("UnknownValue", "<unknown>");
	}

	/** Callback for getting the text that indicates whether the shown device is the platform's default device. */
	FText HandleIsDefaultText() const
	{
		if (DeviceService.IsValid())
		{
			ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

			if (TargetDevice.IsValid() && TargetDevice->IsDefault())
			{
				return LOCTEXT("YesText", "yes");
			}

			return LOCTEXT("NoText", "no");
		}

		return LOCTEXT("UnknownValue", "<unknown>");
	}

	/** Callback for getting the make and model of the shown device. */
	FText HandleOperatingSystemText() const
	{
		if (DeviceService.IsValid())
		{
			ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

			if (TargetDevice.IsValid())
			{
				FString OSName = TargetDevice->GetOperatingSystemName();

				if (!OSName.IsEmpty())
				{
					return FText::FromString(OSName);
				}
			}
		}

		return LOCTEXT("UnknownValue", "<unknown>");
	}

	/** Callback for getting the icon of the device's platform. */
	const FSlateBrush* HandlePlatformIcon() const
	{
		if (DeviceService.IsValid())
		{
			const PlatformInfo::FTargetPlatformInfo* const PlatformInfo = PlatformInfo::FindPlatformInfo(DeviceService->GetDevicePlatformName());
			if(PlatformInfo)
			{
				return FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::XLarge));
			}
		}

		return FStyleDefaults::GetNoBrush();
	}

	/** Callback for getting the name of the device's platform. */
	FText HandlePlatformNameText() const
	{
		if (DeviceService.IsValid())
		{
			FString PlatformName = DeviceService->GetDevicePlatformDisplayName();

			if (!PlatformName.IsEmpty())
			{
				return FText::FromString(PlatformName);
			}
		}

		return LOCTEXT("UnknownValue", "<unknown>");
	}

	/** Callback for getting the status of the device. */
	FText HandleStatusText() const
	{
		if (DeviceService.IsValid())
		{
			ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

			if (TargetDevice.IsValid())
			{
				if (TargetDevice->IsConnected())
				{
					return LOCTEXT("StatusConnected", "Connected");
				}

				return LOCTEXT("StatusDisconnected", "Disconnected");
			}

			return LOCTEXT("StatusUnavailable", "Unavailable");
		}

		return FText::GetEmpty();
	}

private:

	/** The service for the device whose details are being shown. */
	TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe> DeviceService;
};


#undef LOCTEXT_NAMESPACE
