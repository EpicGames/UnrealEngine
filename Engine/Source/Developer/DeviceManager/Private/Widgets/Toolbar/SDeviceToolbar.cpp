// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Toolbar/SDeviceToolbar.h"

#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SBorder.h"

#include "Models/DeviceDetailsCommands.h"
#include "Models/DeviceManagerModel.h"


#define LOCTEXT_NAMESPACE "SDeviceToolbar"


/* SDeviceToolbar interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDeviceToolbar::Construct(const FArguments& InArgs, const TSharedRef<FDeviceManagerModel>& InModel, const TSharedPtr<FUICommandList>& InUICommandList)
{
	Model = InModel;

	// callback for getting the enabled state of the toolbar.
	auto ToolbarIsEnabled = [this]() -> bool {
		return Model->GetSelectedDeviceService().IsValid();
	};

	// create the toolbar
	FToolBarBuilder Toolbar(InUICommandList, FMultiBoxCustomization::None);
	{
		Toolbar.AddToolBarButton(FDeviceDetailsCommands::Get().Claim);
		Toolbar.AddToolBarButton(FDeviceDetailsCommands::Get().Release);
		Toolbar.AddToolBarButton(FDeviceDetailsCommands::Get().Share);
		Toolbar.AddToolBarButton(FDeviceDetailsCommands::Get().Remove);

		Toolbar.AddSeparator();
		Toolbar.AddToolBarButton(FDeviceDetailsCommands::Get().Connect);
		Toolbar.AddToolBarButton(FDeviceDetailsCommands::Get().Disconnect);

		Toolbar.AddSeparator();
		Toolbar.AddToolBarButton(FDeviceDetailsCommands::Get().PowerOn);
		Toolbar.AddToolBarButton(FDeviceDetailsCommands::Get().PowerOff);
		Toolbar.AddToolBarButton(FDeviceDetailsCommands::Get().Reboot);
	}

	// construct children
	ChildSlot
	[
		SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.IsEnabled_Lambda(ToolbarIsEnabled)
			.Padding(0.0f)
			[
				Toolbar.MakeWidget()
			]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
