// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FDeviceManagerModel;
class FUICommandList;


/**
 * Implements the device toolbar widget.
 */
class SDeviceToolbar
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceToolbar) { }
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InModel The view model to use.
	 * @param InUICommandList The UI command list to use.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FDeviceManagerModel>& InModel, const TSharedPtr<FUICommandList>& InUICommandList);

private:

	/** Pointer the device manager's view model. */
	TSharedPtr<FDeviceManagerModel> Model;
};
