// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class ITargetDeviceServiceManager;
class IDeviceManagerCustomPlatformWidgetCreator;
class SDockTab;
class SWidget;
class SWindow;

/**
 * Interface for device manager modules.
 */
class IDeviceManagerModule
	: public IModuleInterface
{
public:

	/**
	 * Create a device manager widget.
	 *
	 * @param DeviceServiceManager The target device manager to use.
	 * @return The new widget.
	 */
	virtual TSharedRef<SWidget> CreateDeviceManager(const TSharedRef<ITargetDeviceServiceManager>& DeviceServiceManager, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow) = 0;

public:

	/** Virtual destructor. */
	virtual ~IDeviceManagerModule() = default;
};
